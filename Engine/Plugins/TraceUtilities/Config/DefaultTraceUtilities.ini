[EditorTraceUtilities]
OpenLiveSesssionOnTraceStart=false
OpenInsightsAfterTrace=false
ShowInExplorerAfterTrace=false