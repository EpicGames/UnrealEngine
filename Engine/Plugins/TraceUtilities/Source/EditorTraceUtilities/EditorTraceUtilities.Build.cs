// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class EditorTraceUtilities : ModuleRules
{
	public EditorTraceUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"MessageLog",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"TraceAnalysis",
				"TraceInsightsFrontend",
				"TraceTools",
				"TraceLog",
				"UATHelper",
			}
		);
	}
}
