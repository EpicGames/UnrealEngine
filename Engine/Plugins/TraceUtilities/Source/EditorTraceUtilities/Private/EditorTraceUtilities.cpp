// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorTraceUtilities.h"

#include "Misc/ConfigContext.h"
#include "ToolMenus.h"
#include "InsightsStatusBar/SInsightsStatusBar.h"

DEFINE_LOG_CATEGORY(LogTraceUtilities)

#define LOCTEXT_NAMESPACE "FEditorTraceUtilitiesModule"

namespace UE::EditorTraceUtilities
{
/** This function will add the SInsightsStatusBarWidget to the Editor's status bar at the bottom ("LevelEditor.StatusBar.ToolBar"). */
void RegisterInsightsStatusWidgetWithToolMenu();

FString FEditorTraceUtilitiesModule::EditorTraceUtilitiesIni;

void FEditorTraceUtilitiesModule::StartupModule()
{
	LLM_SCOPE_BYNAME(TEXT("Insights"));

	RegisterInsightsStatusWidgetWithToolMenu();
	FConfigContext::ReadIntoGConfig().Load(TEXT("TraceUtilities"), EditorTraceUtilitiesIni);
}

void FEditorTraceUtilitiesModule::ShutdownModule()
{
	LLM_SCOPE_BYNAME(TEXT("Insights"));
	UToolMenus::UnRegisterStartupCallback(RegisterStartupCallbackHandle);
}

const FStatusBarTraceSettings& FEditorTraceUtilitiesModule::GetTraceSettings() const
{
	return SInsightsStatusBarWidget::StatusBarTraceSettings;
}
} // namespace UE::EditorTraceUtilities

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(UE::EditorTraceUtilities::FEditorTraceUtilitiesModule, EditorTraceUtilities)