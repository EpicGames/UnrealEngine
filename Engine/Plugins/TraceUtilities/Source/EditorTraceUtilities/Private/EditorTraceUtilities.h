// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IEditorTraceUtilitiesModule.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogTraceUtilities, Log, All)

namespace UE::EditorTraceUtilities
{
class FEditorTraceUtilitiesModule : public IEditorTraceUtilitiesModule
{
public:

	//~ Begin IEditorTraceUtilitiesModule Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual const FStatusBarTraceSettings& GetTraceSettings() const override;
	//~ End IEditorTraceUtilitiesModule Interface

	static const FString& GetTraceUtilitiesIni() { return EditorTraceUtilitiesIni; }

private:
	FDelegateHandle RegisterStartupCallbackHandle;
	static FString EditorTraceUtilitiesIni;
};
} // namespace UE::EditorTraceUtilities