// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"

class FEditorTraceUtilitiesStyle final
	: public FSlateStyleSet
{
public:
	FEditorTraceUtilitiesStyle()
		: FSlateStyleSet("EditorTraceUtilitiesStyle")
	{
		const FVector2D Icon10x10(10.0f, 10.0f);
		const FVector2D Icon16x16(16.0f, 16.0f);
		const FVector2D Icon20x20(20.0f, 20.0f);
		const FVector2D Icon24x24(24.0f, 24.0f);
		const FVector2D Icon32x32(32.0f, 32.0f);
		const FVector2D Icon40x40(40.0f, 40.0f);

		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("/TraceUtilities/Content/Slate"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

		Set("Icons.OpenLiveSession.Menu", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/Session", Icon16x16));

		Set("Icons.UnrealInsights.Menu", new IMAGE_BRUSH_SVG("UnrealInsights_16", Icon16x16));

		Set("Icons.Trace.Menu", new IMAGE_BRUSH_SVG("Trace_16", Icon16x16));
		Set("Icons.Trace.StatusBar", new IMAGE_BRUSH_SVG("Trace_20", Icon16x16));

		Set("Icons.StartTrace.Menu", new IMAGE_BRUSH_SVG("StartTrace_16", Icon16x16));
		Set("Icons.StartTrace.StatusBar", new IMAGE_BRUSH_SVG("StartTrace_20", Icon16x16));

		Set("Icons.TraceSnapshot.Menu", new IMAGE_BRUSH_SVG("TraceSnapshot_16", Icon16x16));
		Set("Icons.TraceSnapshot.StatusBar", new IMAGE_BRUSH_SVG("TraceSnapshot_20", Icon16x16));

		Set("Icons.TraceStore.Menu", new CORE_IMAGE_BRUSH_SVG("Starship/Insights/TraceStore", Icon16x16));
		Set("Icons.File.Menu", new CORE_IMAGE_BRUSH_SVG("Starship/Common/file", Icon16x16));

		Set("Icons.Screenshot.Menu", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/HighResolutionScreenshot", Icon16x16));
		Set("Icons.Bookmark.Menu", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/Bookmarks", Icon16x16));
		
		Set("Icons.TraceServerStart", new CORE_IMAGE_BRUSH_SVG("Starship/Common/play", Icon16x16, FStyleColors::AccentGreen));
		Set("Icons.TraceServerStop", new CORE_IMAGE_BRUSH_SVG("Starship/Common/close", Icon16x16, FStyleColors::AccentRed));

		Set("Icons.PauseTrace.Menu", new IMAGE_BRUSH_SVG("Pause_16", Icon16x16));
		Set("Icons.ResumeTrace.Menu", new CORE_IMAGE_BRUSH_SVG("Starship/Common/play", Icon16x16));

		Set("Icons.BeginRegion.Menu", new CORE_IMAGE_BRUSH_SVG("Starship/Common/play", Icon16x16));
		Set("Icons.EndRegion.Menu", new CORE_IMAGE_BRUSH_SVG("Starship/Common/stop", Icon16x16));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	static FEditorTraceUtilitiesStyle& Get()
	{
		static FEditorTraceUtilitiesStyle Inst;
		return Inst;
	}
	
	~FEditorTraceUtilitiesStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}
};
