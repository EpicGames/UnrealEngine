// Copyright Epic Games, Inc. All Rights Reserved.

#include "SInsightsStatusBar.h"

#include "CoreGlobals.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "Interfaces/IMainFrameModule.h"
#include "Logging/MessageLog.h"
#include "Logging/TokenizedMessage.h"
#include "Math/Color.h"
#include "MessageLogModule.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DateTime.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/MiscTrace.h"
#include "ProfilingDebugging/PlatformEvents.h"
#include "ProfilingDebugging/TraceScreenshot.h"
#include "Styling/StyleColors.h"
#include "ToolMenus.h"
#include "Trace/Detail/Channel.h"
#include "Trace/StoreClient.h"
#include "Trace/Trace.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SBoxPanel.h"

// TraceTools
#include "TraceTools/Interfaces/ITraceToolsModule.h"
#include "TraceTools/Widgets/SToggleTraceButton.h"

// TraceInsightsFrontend
#include "InsightsFrontend/StoreService/TraceServerControl.h"

// EditorTraceUtilities
#include "EditorTraceUtilities.h"
#include "EditorTraceUtilitiesStyle.h"
#include "SRecentTracesList.h"
#include "UnrealInsightsLauncher.h"

#define LOCTEXT_NAMESPACE "InsightsEditor"

namespace UE::EditorTraceUtilities
{

FStatusBarTraceSettings SInsightsStatusBarWidget::StatusBarTraceSettings;

const TCHAR* SInsightsStatusBarWidget::DefaultPreset = TEXT("default");
const TCHAR* SInsightsStatusBarWidget::MemoryPreset = TEXT("default,memory");
const TCHAR* SInsightsStatusBarWidget::TaskGraphPreset = TEXT("default,task");
const TCHAR* SInsightsStatusBarWidget::ContextSwitchesPreset = TEXT("default,contextswitches");

const TCHAR* SInsightsStatusBarWidget::SettingsCategory = TEXT("EditorTraceUtilities");
const TCHAR* SInsightsStatusBarWidget::OpenLiveSessionOnTraceStartSettingName = TEXT("OpenLiveSessionOnTraceStart");
const TCHAR* SInsightsStatusBarWidget::OpenInsightsAfterTraceSettingName = TEXT("OpenInsightsAfterTrace");
const TCHAR* SInsightsStatusBarWidget::TraceRegionSettingName = TEXT("InsightsToolbarTraceRegion");
const TCHAR* SInsightsStatusBarWidget::ShowInExplorerAfterTraceSettingName = TEXT("ShowInExplorerAfterTrace");

////////////////////////////////////////////////////////////////////////////////////////////////////
// FInsightsStatusBarWidgetCommands
////////////////////////////////////////////////////////////////////////////////////////////////////

class FInsightsStatusBarWidgetCommands : public TCommands<FInsightsStatusBarWidgetCommands>
{
public:
	FInsightsStatusBarWidgetCommands()
		: TCommands<FInsightsStatusBarWidgetCommands>(
			TEXT("InsightsStatusBarWidgetCommands"),
			NSLOCTEXT("Contexts", "InsightsStatusBarWidgetCommands", "Insights Status Bar"),
			NAME_None,
			FEditorTraceUtilitiesStyle::Get().GetStyleSetName())
	{
	}

	virtual ~FInsightsStatusBarWidgetCommands()
	{
	}

	// UI_COMMAND takes long for the compiler to optimize
	UE_DISABLE_OPTIMIZATION_SHIP
	virtual void RegisterCommands() override
	{
		UI_COMMAND(Command_TraceScreenshot, "Trace Screenshot", "Takes a screenshot and sends it to the trace.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F9));
		UI_COMMAND(Command_TraceBookmark, "Trace Bookmark", "Traces a bookmark.", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND(Command_TraceToggle, "Toggle Trace", "Toggles trace capture.", EUserInterfaceActionType::Button, FInputChord());
	}
	UE_ENABLE_OPTIMIZATION_SHIP

	TSharedPtr<FUICommandInfo> Command_TraceScreenshot;
	TSharedPtr<FUICommandInfo> Command_TraceBookmark;
	TSharedPtr<FUICommandInfo> Command_TraceToggle;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedRef<SWidget> CreateInsightsStatusBarWidget()
{
	return SNew(SInsightsStatusBarWidget);
}

void RegisterInsightsStatusWidgetWithToolMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(TEXT("LevelEditor.StatusBar.ToolBar"));

	FToolMenuSection& InsightsSection = Menu->AddSection(TEXT("Insights"), FText::GetEmpty(), FToolMenuInsert(NAME_None, EToolMenuInsertType::First));

	InsightsSection.AddEntry(
		FToolMenuEntry::InitWidget(TEXT("InsightsStatusBar"), CreateInsightsStatusBarWidget(), FText::GetEmpty(), true, false)
	);
}

void SInsightsStatusBarWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SInsightsStatusBarWidget::Tick);

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	bShouldUpdateChannels = true;

	if (bShouldTryOpenLiveSession)
	{
		constexpr double Delay = 1.0;
		OpenLiveSessionScheduledTime = InCurrentTime + Delay;
		bShouldTryOpenLiveSession = false;
	}

	if (OpenLiveSessionScheduledTime > 0.0 && OpenLiveSessionScheduledTime <= InCurrentTime)
	{
		OpenLiveSessionScheduledTime = 0.0;
		FUnrealInsightsLauncher::Get()->TryOpenTraceFromDestination(FTraceAuxiliary::GetTraceDestinationString());
	}
}

void SInsightsStatusBarWidget::Construct(const FArguments& InArgs)
{
	FModuleManager::LoadModuleChecked<UE::TraceTools::ITraceToolsModule>("TraceTools");

	this->ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SComboButton)
			.ContentPadding(FMargin(6.0f, 0.0f))
			.MenuPlacement(MenuPlacement_AboveAnchor)
			.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.OnGetMenuContent(this, &SInsightsStatusBarWidget::MakeTraceMenu)
			.HasDownArrow(true)
			.ToolTipText(this, &SInsightsStatusBarWidget::GetTitleToolTipText)
			.ButtonContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 3.0f, 0.0f)
				.AutoWidth()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FEditorTraceUtilitiesStyle::Get().GetBrush("Icons.Trace.StatusBar"))
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Trace", "Trace"))
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DialogButtonText"))
				]
			]
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(UE::TraceTools::SToggleTraceButton)
			.OnToggleTraceRequested(this, &SInsightsStatusBarWidget::ToggleTrace_OnClicked)
			.IsTraceRunning_Lambda([]() { return UE::Trace::IsTracing(); })
			.ButtonSize(UE::TraceTools::SToggleTraceButton::EButtonSize::StatusBar)
			.IsEnabled(this, &SInsightsStatusBarWidget::ToggleTrace_CanExecute)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ContentPadding(FMargin(4.0f, 0.0f, 0.0f, 3.0f))
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Bottom)
			.OnClicked_Lambda([this]() { SaveSnapshot(); return FReply::Handled(); })
			.IsEnabled(this, &SInsightsStatusBarWidget::SaveSnapshot_CanExecute)
			.Content()
			[
				SNew(SImage)
				.DesiredSizeOverride(CoreStyleConstants::Icon16x16)
				.Image(FEditorTraceUtilitiesStyle::Get().GetBrush("Icons.TraceSnapshot.StatusBar"))
				.ToolTipText(LOCTEXT("SaveSnapShot","Snapshot: Save Current Trace Buffer to active destination."))
			]
		]
	];

	if (FTraceAuxiliary::GetConnectionType() == FTraceAuxiliary::EConnectionType::Network)
	{
		StatusBarTraceSettings.TraceDestination = ETraceDestination::TraceStore;
	}
	if (FTraceAuxiliary::GetConnectionType() == FTraceAuxiliary::EConnectionType::File)
	{
		StatusBarTraceSettings.TraceDestination = ETraceDestination::File;
	}

	LogListingName = TEXT("UnrealInsights");
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	if (!MessageLogModule.IsRegisteredLogListing(LogListingName))
	{
		MessageLogModule.RegisterLogListing(LogListingName, LOCTEXT("UnrealInsights", "Unreal Insights"));
	}

	FTraceAuxiliary::OnTraceStarted.AddSP(this, &SInsightsStatusBarWidget::OnTraceStarted);
	FTraceAuxiliary::OnTraceStopped.AddSP(this, &SInsightsStatusBarWidget::OnTraceStopped);
	FTraceAuxiliary::OnSnapshotSaved.AddSP(this, &SInsightsStatusBarWidget::OnSnapshotSaved);

	LiveSessionTracker = MakeShared<FLiveSessionTracker>();

	InitCommandList();
}

TSharedRef<SWidget> SInsightsStatusBarWidget::MakeTraceMenu()
{
	LiveSessionTracker->StartQuery();

	if (ServerControls.IsEmpty())
	{
		ServerControls.Emplace(TEXT("127.0.0.1"), 0, FEditorTraceUtilitiesStyle::Get().GetStyleSetName());
	}

	FMenuBuilder MenuBuilder(true, CommandList.ToSharedRef());

	MenuBuilder.BeginSection("TraceData", LOCTEXT("TraceMenu_Section_Data", "Trace Data"));
	{
		MenuBuilder.AddSubMenu
		(
			LOCTEXT("Channels", "Channels"),
			LOCTEXT("Channels_Desc", "Select what trace channels to enable when tracing."),
			FNewMenuDelegate::CreateSP(this, &SInsightsStatusBarWidget::Channels_BuildMenu),
			false,
			FSlateIcon(FEditorTraceUtilitiesStyle::Get().GetStyleSetName(), ("Icons.Trace.Menu"))
		);

		MenuBuilder.AddMenuEntry
		(
			FInsightsStatusBarWidgetCommands::Get().Command_TraceScreenshot,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>::CreateSP(this, &SInsightsStatusBarWidget::GetTraceScreenshotTooltipText),
			FSlateIcon(FEditorTraceUtilitiesStyle::Get().GetStyleSetName(), "Icons.Screenshot.Menu")
		);

		MenuBuilder.AddMenuEntry
		(
			FInsightsStatusBarWidgetCommands::Get().Command_TraceBookmark,
			NAME_None,
			TAttribute<FText>(),
			TAttribute<FText>::CreateSP(this, &SInsightsStatusBarWidget::GetTraceBookmarkTooltipText),
			FSlateIcon(FEditorTraceUtilitiesStyle::Get().GetStyleSetName(), "Icons.Bookmark.Menu")
		);

		MenuBuilder.AddEditableText(LOCTEXT("TraceRegionNameLabel", "Region Name:"),
			GetTraceRegionNameDesc(),
			FSlateIcon(),
			TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateRaw(this, &SInsightsStatusBarWidget::GetTraceRegionName)),
			FOnTextCommitted::CreateLambda(
				[](const FText& NewRegionName, ETextCommit::Type)
				{
					GConfig->SetText(SettingsCategory, TraceRegionSettingName, NewRegionName, FEditorTraceUtilitiesModule::GetTraceUtilitiesIni());
				}),
			FOnTextChanged(),
			RegionIsActive());

		MenuBuilder.AddMenuEntry(
			GetRegionSwitchLabelText(),
			TAttribute<FText>::CreateSP(this, &SInsightsStatusBarWidget::GetRegionSwitchDescText),
			FSlateIcon(FEditorTraceUtilitiesStyle::Get().GetStyleSetName(), RegionIsActive() ? "Icons.EndRegion.Menu" : "Icons.BeginRegion.Menu"),
			FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::ToggleRegion_Execute),
					  FCanExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::ToggleRegion_CanExecute)),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("StatNamedEventsLabel", "Stat Named Events"),
			LOCTEXT("StatNamedEventsDesc", "Significantly increases the number of CPU profiler events by activating tracing for the named events system."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateLambda([]() { GCycleStatsShouldEmitNamedEvents = GCycleStatsShouldEmitNamedEvents == 0 ? 1 : 0; }),
				      FCanExecuteAction(),
					  FIsActionChecked::CreateLambda([]() { return GCycleStatsShouldEmitNamedEvents > 0; })),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("TraceDestination", LOCTEXT("TraceMenu_Section_Destination", "Trace Destination"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ServerLabel", "Trace Store"),
			LOCTEXT("ServerLabelDesc", "Set the trace store as the trace destination."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::SetTraceDestination_Execute, ETraceDestination::TraceStore),
					  FCanExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::SetTraceDestination_CanExecute),
					  FIsActionChecked::CreateSP(this, &SInsightsStatusBarWidget::SetTraceDestination_IsChecked, ETraceDestination::TraceStore)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("FileLabel", "File"),
			LOCTEXT("FileLabelDesc", "Set file as the trace destination."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::SetTraceDestination_Execute, ETraceDestination::File),
					  FCanExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::SetTraceDestination_CanExecute),
					  FIsActionChecked::CreateSP(this, &SInsightsStatusBarWidget::SetTraceDestination_IsChecked, ETraceDestination::File)),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Tracing", LOCTEXT("TraceMenu_Section_Tracing", "Tracing"));
	{
		MenuBuilder.AddMenuEntry(
			FInsightsStatusBarWidgetCommands::Get().Command_TraceToggle,
			NAME_None,
			TAttribute<FText>::CreateSP(this, &SInsightsStatusBarWidget::GetTraceMenuItemText),
			TAttribute<FText>::CreateSP(this, &SInsightsStatusBarWidget::GetTraceMenuItemTooltipText),
			FSlateIcon(FEditorTraceUtilitiesStyle::Get().GetStyleSetName(), "Icons.StartTrace.Menu")
		);

		if (FTraceAuxiliary::IsPaused())
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ResumeTraceButtonText", "Resume Trace"),
				LOCTEXT("ResumesTraceButtonTooltip", "Enables all channels that were active when tracing was paused."),
				FSlateIcon(FEditorTraceUtilitiesStyle::Get().GetStyleSetName(), "Icons.ResumeTrace.Menu"),
				FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::TogglePauseTrace_OnClicked),
						  FCanExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::PauseTrace_CanExecute)),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}
		else
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("PauseTraceButtonText", "Pause Trace"),
				TAttribute<FText>::CreateSP(this, &SInsightsStatusBarWidget::GetPauseTraceMenuItemTooltipText),
				FSlateIcon(FEditorTraceUtilitiesStyle::Get().GetStyleSetName(), "Icons.PauseTrace.Menu"),
				FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::TogglePauseTrace_OnClicked),
						  FCanExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::PauseTrace_CanExecute)),
				NAME_None,
				EUserInterfaceActionType::Button
			);
		}

		MenuBuilder.AddMenuEntry(
			LOCTEXT("SaveSnapshotLabel", "Save Trace Snapshot"),
			LOCTEXT("SaveSnapshotTooltip", "Save the current trace buffer to the selected trace destination."),
			FSlateIcon(FEditorTraceUtilitiesStyle::Get().GetStyleSetName(), "Icons.TraceSnapshot.Menu"),
			FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::SaveSnapshot),
					  FCanExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::SaveSnapshot_CanExecute)),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Options", LOCTEXT("TraceMenu_Section_Options", "Options"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenLiveSessionOnTraceStart", "Open Live Session on Trace Start"),
			LOCTEXT("OpenLiveSessionOnTraceStartDesc", "When set, the live session will be automatically opened in Unreal Insights when tracing is started.\nThis option will only apply when tracing to the trace store."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::ToggleBooleanSettingValue, OpenLiveSessionOnTraceStartSettingName),
					  FCanExecuteAction(),
					  FIsActionChecked::CreateSP(this, &SInsightsStatusBarWidget::GetBooleanSettingValue, OpenLiveSessionOnTraceStartSettingName)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenInsightsAfterTrace", "Open Insights after Trace"),
			LOCTEXT("OpenInsightsAfterTraceDesc", "When set, the session will be automatically opened in Unreal Insights when tracing is stopped or when a snapshot is saved."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::ToggleBooleanSettingValue, OpenInsightsAfterTraceSettingName),
					  FCanExecuteAction(),
					  FIsActionChecked::CreateSP(this, &SInsightsStatusBarWidget::GetBooleanSettingValue, OpenInsightsAfterTraceSettingName)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("ShowInExplorerAfterTrace", "Show in Explorer after Trace"),
			LOCTEXT("ShowInExplorerAfterTraceDesc", "When set, folder containing the recorded session will be opened automatically when trace is stopped or when a snapshot is saved."),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::ToggleBooleanSettingValue, ShowInExplorerAfterTraceSettingName),
					  FCanExecuteAction(),
					  FIsActionChecked::CreateSP(this, &SInsightsStatusBarWidget::GetBooleanSettingValue, ShowInExplorerAfterTraceSettingName)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Locations", LOCTEXT("TraceMenu_Section_Locations", "Locations"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenTraceStoreLabel", "Open Trace Store Directory"),
			LOCTEXT("OpenTraceStoreTooltip", "Open Trace Store Directory. This is the location where traces saved to the trace server are stored."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FolderOpen"),
			FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::OpenTraceStoreDirectory_OnClicked)),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenProfilingDirectoryLabel", "Open Profiling Directory"),
			LOCTEXT("OpenProfilingDirectoryTooltip", "Opens the profiling directory of the current project. This is the location where traces to file are stored."),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.FolderOpen"),
			FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::OpenProfilingDirectory_OnClicked)),
			NAME_None,
			EUserInterfaceActionType::Button
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Insights", LOCTEXT("TraceMenu_Section_Insights", "Insights"));
	{
		MenuBuilder.AddSubMenu(
			LOCTEXT("ServerControlLabel", "Unreal Trace Server"),
			LOCTEXT("ServerControlTooltip", "Info and controls for the Unreal Trace Server instances"),
			FNewMenuDelegate::CreateLambda(
				[this](FMenuBuilder& MenuBuilder)
				{
					for (auto& ServerControl : ServerControls)
					{
						ServerControl.MakeMenu(MenuBuilder);
					}
				}),
			false,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Server")
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenInsightsLabel", "Unreal Insights (Session Browser)"),
			LOCTEXT("OpenInsightsTooltip", "Launch the Unreal Insights Session Browser."),
			FSlateIcon(FEditorTraceUtilitiesStyle::Get().GetStyleSetName(), "Icons.UnrealInsights.Menu"),
			FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::LaunchUnrealInsights_OnClicked)),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("OpenLiveSessionLabel", "Open Live Session"),
			LOCTEXT("OpenLiveSessionTooltip", "Opening the live session is possible only while tracing to the trace store."),
			FSlateIcon(FEditorTraceUtilitiesStyle::Get().GetStyleSetName(), "Icons.OpenLiveSession.Menu"),
			FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::OpenLiveSession_OnClicked),
					  FCanExecuteAction::CreateLambda([this]() { return FTraceAuxiliary::GetConnectionType() == FTraceAuxiliary::EConnectionType::Network; })),
			NAME_None,
			EUserInterfaceActionType::Button
		);

		MenuBuilder.AddSubMenu
		(
			LOCTEXT("RecentTracesLabel", "Recent Traces"),
			LOCTEXT("RecentTracesTooltip", "Open the latest traces recorded to the trace store or as a file."),
			FNewMenuDelegate::CreateSP(this, &SInsightsStatusBarWidget::Traces_BuildMenu),
			false,
			FSlateIcon(FEditorTraceUtilitiesStyle::Get().GetStyleSetName(), ("Icons.Trace.Menu"))
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SInsightsStatusBarWidget::Channels_BuildMenu(FMenuBuilder& MenuBuilder)
{
	CreateChannelsInfo();

	MenuBuilder.BeginSection("Channels", LOCTEXT("Channels", "Channels"));
	{
		for (int32 Index = 0; Index < ChannelsInfo.Num(); ++Index)
		{
			const FChannelData& Data = ChannelsInfo[Index];

			FString ChannelDisplayName = Data.Name;
			ChannelDisplayName.RemoveFromEnd(TEXT("Channel"), 7);

			FText ChannelTooltip = FText::FromString(Data.Desc);
			if (Data.bIsReadOnly)
			{
				ChannelTooltip = FText::Format(LOCTEXT("ChannelTooltipFmt", "{0}{1} This channel is readonly and can only be enabled from the command line."),
					ChannelTooltip, Data.Desc.EndsWith(".") ? FText() : FText::FromString("."));
			}

			MenuBuilder.AddMenuEntry(
				FText::FromString(ChannelDisplayName),
				ChannelTooltip,
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::ToggleChannel_Execute, Index),
						  FCanExecuteAction::CreateLambda([Value = !Data.bIsReadOnly]() { return Value; }),
						  FIsActionChecked::CreateSP(this, &SInsightsStatusBarWidget::ToggleChannel_IsChecked, Index)),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
		}
	}
	MenuBuilder.EndSection();
}

void SInsightsStatusBarWidget::Traces_BuildMenu(FMenuBuilder& MenuBuilder)
{
	CacheTraceStorePath();
	PopulateRecentTracesList();

	for (int32 Index = 0; Index < Traces.Num(); ++Index)
	{
		MenuBuilder.AddMenuEntry(
			FUIAction(FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::OpenTrace, Index),
					  FCanExecuteAction()),
			SNew(SRecentTracesListEntry, Traces[Index], TraceStorePath, LiveSessionTracker),
			NAME_None,
			FText::FromString(Traces[Index]->FilePath),
			EUserInterfaceActionType::Button);
	}
}

void SInsightsStatusBarWidget::InitCommandList()
{
	FInsightsStatusBarWidgetCommands::Register();

	CommandList = MakeShared<FUICommandList>();

	CommandList->MapAction(FInsightsStatusBarWidgetCommands::Get().Command_TraceScreenshot,
		FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::TraceScreenshot_Execute),
		FCanExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::TraceScreenshot_CanExecute));

	CommandList->MapAction(FInsightsStatusBarWidgetCommands::Get().Command_TraceBookmark,
		FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::TraceBookmark_Execute),
		FCanExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::TraceBookmark_CanExecute));

	CommandList->MapAction(FInsightsStatusBarWidgetCommands::Get().Command_TraceToggle,
		FExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::ToggleTrace_OnClicked),
		FCanExecuteAction::CreateSP(this, &SInsightsStatusBarWidget::ToggleTrace_CanExecute));

	// Ensures the actions are globally accessible.
	IMainFrameModule& MainFrame = FModuleManager::Get().LoadModuleChecked<IMainFrameModule>("MainFrame");
	MainFrame.GetMainFrameCommandBindings()->Append(CommandList.ToSharedRef());
}

FText SInsightsStatusBarWidget::GetTitleToolTipText() const
{
#if UE_TRACE_MINIMAL_ENABLED
	FTextBuilder DescBuilder;

	const FString Dest = FTraceAuxiliary::GetTraceDestinationString();

	if (!Dest.IsEmpty())
	{
		DescBuilder.AppendLineFormat(LOCTEXT("TracingToText", "Tracing to: {0}"), FText::FromString(Dest));
	}
	else if (UE::Trace::IsTracing())
	{
		DescBuilder.AppendLine(LOCTEXT("TracingToUnknownText", "Tracing to unknown target (externally set)."));
	}
	else
	{
		DescBuilder.AppendLine(LOCTEXT("NotTracingText","Not currently tracing."));
	}

	return DescBuilder.ToText();
#else
	return LOCTEXT("TraceStatusDisabled", "Trace system is disabled at compile time. Check the UE_TRACE_MINIMAL_ENABLED define.");
#endif
}

void SInsightsStatusBarWidget::LogMessage(const FText& Text)
{
	FMessageLog ReportMessageLog(LogListingName);
	TSharedRef<FTokenizedMessage> TokenizedMessage = FTokenizedMessage::Create(EMessageSeverity::Error, Text);
	ReportMessageLog.AddMessage(TokenizedMessage);
	ReportMessageLog.Notify();
}

void SInsightsStatusBarWidget::ShowNotification(const FText& Text, const FText& SubText)
{
	FNotificationInfo Info(Text);
	Info.bFireAndForget = true;
	Info.FadeOutDuration = 1.0f;
	Info.ExpireDuration = 4.0f;

	Info.SubText = SubText;

	FSlateNotificationManager::Get().AddNotification(Info);
}

void SInsightsStatusBarWidget::LaunchUnrealInsights_OnClicked()
{
	FUnrealInsightsLauncher::Get()->StartUnrealInsights(FUnrealInsightsLauncher::Get()->GetInsightsApplicationPath());
}

void SInsightsStatusBarWidget::OpenLiveSession_OnClicked()
{
	OpenLiveSession(FTraceAuxiliary::GetTraceDestinationString());
}

void SInsightsStatusBarWidget::OpenLiveSession(const FString& InTraceDestination)
{
	FUnrealInsightsLauncher::Get()->TryOpenTraceFromDestination(InTraceDestination);
}

void SInsightsStatusBarWidget::OpenProfilingDirectory_OnClicked()
{
	OpenProfilingDirectory();
}

void SInsightsStatusBarWidget::OpenProfilingDirectory()
{
	FString FullPath(FPaths::ConvertRelativePathToFull(FPaths::ProfilingDir()));

	if (!IFileManager::Get().DirectoryExists(*FullPath))
	{
		IFileManager::Get().MakeDirectory(*FullPath);
	}

	FPlatformProcess::ExploreFolder(*FullPath);
}

void SInsightsStatusBarWidget::OpenTraceStoreDirectory_OnClicked()
{
	OpenTraceStoreDirectory(ESelectLatestTraceCriteria::None);
}

void SInsightsStatusBarWidget::OpenTraceStoreDirectory(ESelectLatestTraceCriteria Criteria)
{
	CacheTraceStorePath();
	FString Path = TraceStorePath;

	if (Criteria != ESelectLatestTraceCriteria::None)
	{
		Path = GetLatestTraceFileFromFolder(TraceStorePath, Criteria);
	}

	FPlatformProcess::ExploreFolder(*Path);
}

void SInsightsStatusBarWidget::OpenLatestTraceFromFolder(const FString& InFolder, ESelectLatestTraceCriteria InCriteria)
{
	FString Path = GetLatestTraceFileFromFolder(InFolder, InCriteria);

	if (!Path.IsEmpty())
	{
		FUnrealInsightsLauncher::Get()->TryOpenTraceFromDestination(Path);
	}
}

FString SInsightsStatusBarWidget::GetLatestTraceFileFromFolder(const FString& InFolder, ESelectLatestTraceCriteria InCriteria)
{
	FString Result;
	FString MostRecentTraceName;
	FDateTime LatestDateTime;

	auto Visitor = [&MostRecentTraceName, &LatestDateTime, InCriteria](const TCHAR* Filename, const FFileStatData& StatData)
	{
		if (FPaths::GetExtension(Filename) == TEXT("utrace"))
		{
			if (InCriteria == ESelectLatestTraceCriteria::ModifiedTime && LatestDateTime < StatData.ModificationTime)
			{
				LatestDateTime = StatData.ModificationTime;
				MostRecentTraceName = Filename;
			}

			if (InCriteria == ESelectLatestTraceCriteria::CreatedTime && LatestDateTime < StatData.CreationTime)
			{
				LatestDateTime = StatData.CreationTime;
				MostRecentTraceName = Filename;
			}
		}
		return true;
	};

	IFileManager::Get().IterateDirectoryStat(*InFolder, Visitor);

	if (!MostRecentTraceName.IsEmpty())
	{
		Result = FPaths::ConvertRelativePathToFull(MostRecentTraceName);
	}

	return Result;
}

void SInsightsStatusBarWidget::SetTraceDestination_Execute(ETraceDestination InDestination)
{
	StatusBarTraceSettings.TraceDestination = InDestination;
}

bool SInsightsStatusBarWidget::SetTraceDestination_IsChecked(ETraceDestination InDestination)
{
	return InDestination == StatusBarTraceSettings.TraceDestination;
}

bool SInsightsStatusBarWidget::SetTraceDestination_CanExecute()
{
	return !UE::Trace::IsTracing();
}

void SInsightsStatusBarWidget::SaveSnapshot()
{
	if (StatusBarTraceSettings.TraceDestination == ETraceDestination::File)
	{
		const bool bResult = FTraceAuxiliary::WriteSnapshot(nullptr);
		if (bResult)
		{
			ShowNotification(LOCTEXT("SnapshotSavedHeading", "Insights Snapshot saved."), LOCTEXT("SnapshotSavedFileText", "A snapshot .utrace with the most recent events has been saved to your Saved/Profiling/ directory."));
			return;
		}
	}
	else
	{
		const bool bResult = FTraceAuxiliary::SendSnapshot(nullptr);
		if (bResult)
		{
			ShowNotification(LOCTEXT("SnapshotSavedHeading", "Insights Snapshot saved."), LOCTEXT("SnapshotSavedServerText", "A snapshot .utrace with the most recent events has been saved to your trace server."));
			return;
		}
	}
	LogMessage(LOCTEXT("SnapshotSavedError", "The snapshot could not be saved."));
}

bool SInsightsStatusBarWidget::SaveSnapshot_CanExecute() const
{
#if UE_TRACE_MINIMAL_ENABLED
	return true;
#else
	return false;
#endif
}

FText SInsightsStatusBarWidget::GetTraceMenuItemText() const
{
	if (UE::Trace::IsTracing())
	{
		return LOCTEXT("StopTraceButtonText", "Stop Trace");
	}

	return LOCTEXT("StartTraceButtonText", "Start Trace");
}

FText SInsightsStatusBarWidget::GetTraceMenuItemTooltipText() const
{
#if UE_TRACE_MINIMAL_ENABLED
	if (UE::Trace::IsTracing())
	{
		return LOCTEXT("StopTraceButtonTooltip", "Stop tracing");
	}

	return LOCTEXT("StartTraceButtonTooltip", "Start tracing to the selected trace destination.");
#else
	return LOCTEXT("StartTraceDisabledButtonTooltip", "Trace system is disabled at compile time. Check the UE_TRACE_MINIMAL_ENABLED define.");
#endif
}

bool SInsightsStatusBarWidget::ToggleTrace_CanExecute() const
{
#if UE_TRACE_MINIMAL_ENABLED
	return true;
#else
	return false;
#endif
}

void SInsightsStatusBarWidget::ToggleTrace_OnClicked()
{
	if (UE::Trace::IsTracing())
	{
		FString TraceDestinationStr = FTraceAuxiliary::GetTraceDestinationString();
		bool bResult = FTraceAuxiliary::Stop();

		if (bResult)
		{
			UE_LOGF(LogTraceUtilities, Log, "Stopped tracing to %ls.", *TraceDestinationStr);
		}
		else
		{
			LogMessage(LOCTEXT("TraceStopFailedMsg", "Failed to stop tracing!"));
		}
	}
	else
	{
		bool bResult = StartTracing();
		if (bResult)
		{
			FString TraceDestinationStr = FTraceAuxiliary::GetTraceDestinationString();
			if (TraceDestinationStr.IsEmpty())
			{
				TraceDestinationStr = TEXT("External Target");
			}

			UE_LOGF(LogTraceUtilities, Log, "Trace started. Trace is now active and saving to the following location (file or trace store): %ls", *TraceDestinationStr);
		}
		else
		{
			if (StatusBarTraceSettings.TraceDestination == ETraceDestination::TraceStore)
			{
				LogMessage(LOCTEXT("TraceFailedToStartTraceStoreMsg", "Trace failed to start. The Unreal Trace Server may not be running. Restart it from the Trace status bar menu > Unreal Trace Server > Start."));
			}
			else
			{
				LogMessage(LOCTEXT("TraceFailedToStartFileMsg", "Trace failed to start. Verify the profiling directory is accessible and writable."));
			}
		}
	}
}

bool SInsightsStatusBarWidget::PauseTrace_CanExecute()
{
	return UE::Trace::IsTracing();
}

FText SInsightsStatusBarWidget::GetPauseTraceMenuItemTooltipText() const
{
	if (!UE::Trace::IsTracing())
	{
		return LOCTEXT("PauseTraceDisabledButtonTooltip", "Tracing must be running to enable the pause functionality.");
	}

	return LOCTEXT("PauseTraceButtonTooltip", "Disables all enabled trace channels. The same channels will be re-enabled when tracing is resumed.");
}

void SInsightsStatusBarWidget::TogglePauseTrace_OnClicked()
{
	if (!UE::Trace::IsTracing())
	{
		return;
	}

	if (FTraceAuxiliary::IsPaused())
	{
		FTraceAuxiliary::Resume();
	}
	else
	{
		FTraceAuxiliary::Pause();
	}
}

bool SInsightsStatusBarWidget::StartTracing()
{
	if (StatusBarTraceSettings.TraceDestination == ETraceDestination::TraceStore)
	{
		return FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::Network, TEXT("localhost"), nullptr);
	}
	else if (StatusBarTraceSettings.TraceDestination == ETraceDestination::File)
	{
		return FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::File, nullptr, nullptr);
	}

	return false;
}

bool SInsightsStatusBarWidget::GetBooleanSettingValue(const TCHAR* InSettingName)
{
	bool Value = false;
	GConfig->GetBool(SettingsCategory, InSettingName, Value, FEditorTraceUtilitiesModule::GetTraceUtilitiesIni());
	return Value;
}

void SInsightsStatusBarWidget::ToggleBooleanSettingValue(const TCHAR* InSettingName)
{
	bool Value = false;
	GConfig->GetBool(SettingsCategory, InSettingName, Value, FEditorTraceUtilitiesModule::GetTraceUtilitiesIni());
	GConfig->SetBool(SettingsCategory, InSettingName, !Value, FEditorTraceUtilitiesModule::GetTraceUtilitiesIni());
}

void SInsightsStatusBarWidget::OnTraceStarted(FTraceAuxiliary::EConnectionType InTraceType, const FString& InTraceDestination)
{
	if (InTraceType == FTraceAuxiliary::EConnectionType::Network && GetBooleanSettingValue(OpenLiveSessionOnTraceStartSettingName))
	{
		bShouldTryOpenLiveSession = true;
	}
}

void SInsightsStatusBarWidget::OnTraceStopped(FTraceAuxiliary::EConnectionType InTraceType, const FString& InTraceDestination)
{
	if (GetBooleanSettingValue(OpenInsightsAfterTraceSettingName))
	{
		OpenLiveSession(InTraceDestination);
	}
	if (GetBooleanSettingValue(ShowInExplorerAfterTraceSettingName))
	{
		if (InTraceType == FTraceAuxiliary::EConnectionType::Network)
		{
			OpenTraceStoreDirectory(ESelectLatestTraceCriteria::ModifiedTime);
		}
		else if (InTraceType == FTraceAuxiliary::EConnectionType::File)
		{
			FPlatformProcess::ExploreFolder(*InTraceDestination);
		}
	}
}

void SInsightsStatusBarWidget::OnSnapshotSaved(FTraceAuxiliary::EConnectionType InTraceType, const FString& InTraceDestination)
{
	if (GetBooleanSettingValue(OpenInsightsAfterTraceSettingName))
	{
		if (InTraceType == FTraceAuxiliary::EConnectionType::Network)
		{
			CacheTraceStorePath();
			OpenLatestTraceFromFolder(TraceStorePath, ESelectLatestTraceCriteria::CreatedTime);
		}
		else if (InTraceType == FTraceAuxiliary::EConnectionType::File)
		{
			FUnrealInsightsLauncher::Get()->TryOpenTraceFromDestination(InTraceDestination);
		}
	}
	if (GetBooleanSettingValue(ShowInExplorerAfterTraceSettingName))
	{
		if (InTraceType == FTraceAuxiliary::EConnectionType::Network)
		{
			OpenTraceStoreDirectory(ESelectLatestTraceCriteria::CreatedTime);
		}
		else if (InTraceType == FTraceAuxiliary::EConnectionType::File)
		{
			FPlatformProcess::ExploreFolder(*InTraceDestination);
		}
	}
}

void SInsightsStatusBarWidget::CacheTraceStorePath()
{
	if (TraceStorePath.IsEmpty())
	{
		using UE::Trace::FStoreClient;
		TUniquePtr<FStoreClient> StoreClient = FStoreClient::Create(TEXT("localhost"));

		if (!StoreClient)
		{
			LogMessage(LOCTEXT("FailedConnectionToStoreMsg", "Failed to connect to the store client."));
			return;
		}

		const FStoreClient::FStatus* Status = StoreClient->GetStatus();
		if (!Status)
		{
			LogMessage(LOCTEXT("FailedToGetStoreStatusMsg", "Failed to get the status of the store client."));
			return;
		}
		TraceStorePath = FString(Status->GetStoreDir());
	}
}

void SInsightsStatusBarWidget::CreateChannelsInfo()
{
#if UE_TRACE_MINIMAL_ENABLED
	ChannelsInfo.Empty();

	UE::Trace::EnumerateChannels([](const UE::Trace::FChannelInfo& Info, void* User)
	{
		TArray<FChannelData>* Channels = (TArray<FChannelData>*) User;

		FChannelData NewChannelInfo;
		NewChannelInfo.Name = Info.Name;
		NewChannelInfo.Desc = Info.Desc;
		NewChannelInfo.bIsEnabled = Info.bIsEnabled;
		NewChannelInfo.bIsReadOnly = Info.bIsReadOnly;

		Channels->Add(NewChannelInfo);
		return true;
	}, &ChannelsInfo);

	Algo::SortBy(ChannelsInfo,
		[](const FChannelData& Entry)
		{
			return Entry.Name;
		},
		[](const FString& Rhs, const FString& Lhs)
		{
			return Rhs.Compare(Lhs) < 0;
		});
#endif
}

void SInsightsStatusBarWidget::UpdateChannelsInfo()
{
#if UE_TRACE_MINIMAL_ENABLED
	UE::Trace::EnumerateChannels([](const UE::Trace::FChannelInfo& Info, void* User)
	{
		TArray<FChannelData>* Channels = (TArray<FChannelData>*) User;

		FString Name = Info.Name;
		int32 Index = Algo::BinarySearchBy(*Channels, Name,
			[](const FChannelData& Entry)
			{
				return Entry.Name;
			},
			[](const FString& Rhs, const FString& Lhs)
			{
				return Rhs.Compare(Lhs) < 0;
			});

		if (Index != INDEX_NONE)
		{
			FChannelData& Data = (*Channels)[Index];
			Data.bIsEnabled = Info.bIsEnabled;
		}

		return true;
	}, &ChannelsInfo);

	bShouldUpdateChannels = false;
#endif
}

void SInsightsStatusBarWidget::ToggleChannel_Execute(int32 Index)
{
	if (Index < ChannelsInfo.Num())
	{
		const FString& ChannelName = ChannelsInfo[Index].Name;
		bool bChannelShouldBeEnabled = !ChannelsInfo[Index].bIsEnabled;
		UE::Trace::ToggleChannel(*ChannelName, bChannelShouldBeEnabled);
	}
}

bool SInsightsStatusBarWidget::ToggleChannel_IsChecked(int32 Index)
{
	if (bShouldUpdateChannels)
	{
		UpdateChannelsInfo();
	}

	if (Index < ChannelsInfo.Num())
	{
		return ChannelsInfo[Index].bIsEnabled;
	}

	return false;
}

bool SInsightsStatusBarWidget::TraceScreenshot_CanExecute() const
{
	return SHOULD_TRACE_SCREENSHOT();
}

void SInsightsStatusBarWidget::TraceScreenshot_Execute()
{
#if UE_SCREENSHOT_TRACE_ENABLED
	FTraceScreenshot::RequestScreenshot(TEXT(""), false);
#endif
}

bool SInsightsStatusBarWidget::TraceBookmark_CanExecute() const
{
	return SHOULD_TRACE_BOOKMARK();
}

void SInsightsStatusBarWidget::TraceBookmark_Execute()
{
	const FString Bookmark = FDateTime::Now().ToString(TEXT("Bookmark_%Y%m%d_%H%M%S"));
	TRACE_BOOKMARK(TEXT("%s"), *Bookmark);
}

FText SInsightsStatusBarWidget::GetTraceRegionName()
{
	return GConfig->GetTextOrDefault(SettingsCategory, TraceRegionSettingName, FText::FromString("ToolbarCustomRegion"), FEditorTraceUtilitiesModule::GetTraceUtilitiesIni());
}

FText SInsightsStatusBarWidget::GetTraceRegionNameDesc()
{
	return RegionIsActive() ?
		LOCTEXT("TraceRegionNameDesc_Active", "The name of the region to start or stop. Cannot be edited while a region is active.") :
		LOCTEXT("TraceRegionNameDesc_Inactive", "The name of the region to start or stop.");
}

void SInsightsStatusBarWidget::ToggleRegion_Execute()
{
	if (RegionIsActive())
	{
		TRACE_END_REGION_WITH_ID(RegionId);
		RegionId = 0;
	}
	else
	{
		RegionId = TRACE_BEGIN_REGION_WITH_ID(*GetTraceRegionName().ToString());
	}
}

bool SInsightsStatusBarWidget::ToggleRegion_CanExecute() const
{
	return SHOULD_TRACE_REGION();
}

bool SInsightsStatusBarWidget::RegionIsActive() const
{
	return RegionId > 0;
}

FText SInsightsStatusBarWidget::GetRegionSwitchLabelText() const
{
	if (RegionIsActive())
	{
		return LOCTEXT("TraceEndRegionLabel", "End Region");
	}
	return LOCTEXT("TraceBeginRegionLabel", "Begin Region");
}

FText SInsightsStatusBarWidget::GetRegionSwitchDescText() const
{
	if (!ToggleRegion_CanExecute())
	{
		return LOCTEXT("TraceRegionDisabledDesc", "Regions can only be traced when the Region channel in enabled.");
	}

	if (RegionIsActive())
	{
		return LOCTEXT("TraceEndRegionDesc", "Marks the ending of a trace region with the name input above.");
	}

	return LOCTEXT("TraceBeginRegionDesc", "Marks the beginning of a trace region with the name input above.");
}

void SInsightsStatusBarWidget::PopulateRecentTracesList()
{
	Traces.Empty();
	bool bIsFromTraceStore = true;
	auto Visitor = [this, &bIsFromTraceStore](const TCHAR* Filename, const FFileStatData& StatData)
	{
		if (FPaths::GetExtension(Filename) == TEXT("utrace"))
		{
			TSharedPtr<FTraceFileInfo> TraceInfo = MakeShared<FTraceFileInfo>();
			TraceInfo->FilePath = Filename;
			TraceInfo->ModifiedTime = StatData.ModificationTime;
			TraceInfo->bIsFromTraceStore = bIsFromTraceStore;

			Traces.Add(TraceInfo);
		}

		return true;
	};

	IFileManager::Get().IterateDirectoryStat(*TraceStorePath, Visitor);

	bIsFromTraceStore = false;
	IFileManager::Get().IterateDirectoryStat(*FPaths::ProfilingDir(), Visitor);

	Algo::SortBy(Traces, [](const TSharedPtr<FTraceFileInfo>& TraceInfo) { return *TraceInfo; });

	constexpr int32 MaxRecentTraces = 15;
	if (Traces.Num() > MaxRecentTraces)
	{
		Traces.RemoveAt(MaxRecentTraces, Traces.Num() - MaxRecentTraces);
	}
}

void SInsightsStatusBarWidget::OpenTrace(int32 Index)
{
	if (Index < Traces.Num())
	{
		if (FTraceAuxiliary::GetConnectionType() == FTraceAuxiliary::EConnectionType::Network)
		{
			if (LiveSessionTracker->HasData())
			{
				const FLiveSessionsMap& Sessions = LiveSessionTracker->GetLiveSessions();
				FString FileName = FPaths::GetBaseFilename(Traces[Index]->FilePath);
				const uint32* TraceId = Sessions.Find(FileName);

				if (TraceId)
				{
					FUnrealInsightsLauncher::Get()->OpenRemoteTrace(TEXT("localhost"), uint16(LiveSessionTracker->GetStorePort()), *TraceId);
					return;
				}
			}
		}

		FUnrealInsightsLauncher::Get()->TryOpenTraceFromDestination(Traces[Index]->FilePath);
	}
}

FText SInsightsStatusBarWidget::GetTraceScreenshotTooltipText() const
{
	if (TraceScreenshot_CanExecute())
	{
		return LOCTEXT("TraceScreenshotTooltip1", "Takes a screenshot and sends it to the trace.");
	}
	else
	{
		return LOCTEXT("TraceScreenshotTooltip2", "Screenshots can only be traced when the Screenshot channel is enabled.");
	}
}

FText SInsightsStatusBarWidget::GetTraceBookmarkTooltipText() const
{
	if (TraceBookmark_CanExecute())
	{
		return LOCTEXT("TraceBookmarkTooltip1", "Traces a bookmark.");
	}
	else
	{
		return LOCTEXT("TraceBookmarkTooltip2", "Bookmarks can only be traced when the Bookmark channel is enabled.");
	}
}

} // namespace UE::EditorTraceUtilities

#undef LOCTEXT_NAMESPACE
