// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorTraceUtilities.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"

class FLiveSessionTracker;
class FMenuBuilder;
class FUICommandList;
namespace UE::Insights { class FTraceServerControl; }

TSharedRef<SWidget> CreateInsightsStatusBarWidget();

namespace UE::EditorTraceUtilities
{
struct FTraceFileInfo
{
	FString FilePath;
	FDateTime ModifiedTime;
	bool bIsFromTraceStore;

	bool operator <(const FTraceFileInfo& rhs)
	{
		return this->ModifiedTime > rhs.ModifiedTime;
	}
};

/**
 *  Status bar widget for Unreal Insights.
 *  Shows buttons to start tracing either to a file or to the trace store and allows saving a snapshot to file.
 */
class SInsightsStatusBarWidget : public SCompoundWidget
{
	struct FChannelData
	{
		FString Name;
		FString Desc;
		bool bIsEnabled = false;
		bool bIsReadOnly = false;
	};

	enum class ESelectLatestTraceCriteria : uint32
	{
		None,
		CreatedTime,
		ModifiedTime,
	};

public:

	/** Settings this widget uses. */
	static FStatusBarTraceSettings StatusBarTraceSettings;
	
	SLATE_BEGIN_ARGS(SInsightsStatusBarWidget) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	FText	GetTitleToolTipText() const;
	
	void LaunchUnrealInsights_OnClicked();
	
	void OpenLiveSession_OnClicked();
	void OpenLiveSession(const FString& InTraceDestination);

	void OpenProfilingDirectory_OnClicked();
	void OpenProfilingDirectory();

	void OpenTraceStoreDirectory_OnClicked();
	void OpenTraceStoreDirectory(ESelectLatestTraceCriteria Criteria);

	void OpenLatestTraceFromFolder(const FString& InFolder, ESelectLatestTraceCriteria InCriteria);
	FString GetLatestTraceFileFromFolder(const FString& InFolder, ESelectLatestTraceCriteria InCriteria);

	void SetTraceDestination_Execute(ETraceDestination InDestination);
	bool SetTraceDestination_CanExecute();
	bool SetTraceDestination_IsChecked(ETraceDestination InDestination);

	void SaveSnapshot();
	bool SaveSnapshot_CanExecute() const;

	FText GetTraceMenuItemText() const;
	FText GetTraceMenuItemTooltipText() const;

	void ToggleTrace_OnClicked();
	bool ToggleTrace_CanExecute() const;

	bool PauseTrace_CanExecute();
	FText GetPauseTraceMenuItemTooltipText() const;
	void TogglePauseTrace_OnClicked();

	bool StartTracing();

	TSharedRef<SWidget> MakeTraceMenu();
	void Channels_BuildMenu(FMenuBuilder& MenuBuilder);
	void Traces_BuildMenu(FMenuBuilder& MenuBuilder);

	void LogMessage(const FText& Text);
	void ShowNotification(const FText& Text, const FText& SubText);

	bool GetBooleanSettingValue(const TCHAR* InSettingName);
	void ToggleBooleanSettingValue(const TCHAR* InSettingName);

	void OnTraceStarted(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);
	void OnTraceStopped(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);
	void OnSnapshotSaved(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);

	void CacheTraceStorePath();

	void ToggleChannel_Execute(int32 Index);
	bool ToggleChannel_IsChecked(int32 Index);

	void CreateChannelsInfo();
	void UpdateChannelsInfo();

	void InitCommandList();

	bool TraceScreenshot_CanExecute() const;
	void TraceScreenshot_Execute();
	FText GetTraceScreenshotTooltipText() const;

	bool TraceBookmark_CanExecute() const;
	void TraceBookmark_Execute();
	FText GetTraceBookmarkTooltipText() const;
	
	FText GetTraceRegionName();
	FText TraceRegionName = FText();
	FText GetTraceRegionNameDesc();

	void ToggleRegion_Execute();
	bool ToggleRegion_CanExecute() const;
	bool RegionIsActive() const;
	FText GetRegionSwitchLabelText() const;
	FText GetRegionSwitchDescText() const;
	
	void PopulateRecentTracesList();

	void OpenTrace(int32 Index);

private:
	static const TCHAR* DefaultPreset;
	static const TCHAR* MemoryPreset;
	static const TCHAR* TaskGraphPreset;
	static const TCHAR* ContextSwitchesPreset;

	static const TCHAR* SettingsCategory;
	static const TCHAR* OpenLiveSessionOnTraceStartSettingName;
	static const TCHAR* OpenInsightsAfterTraceSettingName;
	static const TCHAR* TraceRegionSettingName;
	static const TCHAR* ShowInExplorerAfterTraceSettingName;

	bool bIsTraceRecordButtonHovered = false;
	mutable double ConnectionStartTime = 0.0f;

	FString TraceStorePath;

	TArray<FChannelData> ChannelsInfo;
	bool bShouldUpdateChannels = false;

	TSharedPtr<FLiveSessionTracker> LiveSessionTracker;

	TSharedPtr<FUICommandList> CommandList;
	
	TArray<UE::Insights::FTraceServerControl> ServerControls;

	TArray<TSharedPtr<FTraceFileInfo>> Traces;
	FName LogListingName;

	uint64 RegionId = 0;

	bool bShouldTryOpenLiveSession = false;
	double OpenLiveSessionScheduledTime = 0.0f;
};
}
