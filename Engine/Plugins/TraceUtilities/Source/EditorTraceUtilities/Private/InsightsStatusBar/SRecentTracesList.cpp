// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRecentTracesList.h"

#include "EditorTraceUtilitiesStyle.h"
#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "SInsightsStatusBar.h"
#include "UnrealInsightsLauncher.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "RecentTracesList"

namespace UE::EditorTraceUtilities
{
void SRecentTracesListEntry::Construct(const FArguments& InArgs, TSharedPtr<FTraceFileInfo> InTrace, const FString& InStorePath, TSharedPtr<FLiveSessionTracker> InLiveSessionTracker)
{
	TraceInfo = InTrace;
	StorePath = InStorePath;
	LiveSessionTracker = InLiveSessionTracker;
	TraceName = FPaths::GetBaseFilename(TraceInfo->FilePath);

	TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	int32 LastCharacter = FontMeasureService->FindLastWholeCharacterIndexBeforeOffset(TraceName, FAppStyle::GetFontStyle("Menu.Label"), 180);
	if (LastCharacter < TraceName.Len() - 1 && LastCharacter > 3)
	{
		TraceName.LeftInline(LastCharacter - 3);
		TraceName += TEXT("...");
	}

	FString TooltipText = TraceInfo->FilePath;
	FPaths::NormalizeFilename(TooltipText);
	TooltipText = FPaths::ConvertRelativePathToFull(TooltipText);

	const FSlateBrush* TraceLocationIcon;
	FText TraceLocationTooltip;

	if (TraceInfo->bIsFromTraceStore)
	{
		TraceLocationIcon = FEditorTraceUtilitiesStyle::Get().GetBrush("Icons.TraceStore.Menu");
		TraceLocationTooltip = LOCTEXT("TraceStoreLocationTooltip", "This trace is located in the trace store.");
	}
	else
	{
		TraceLocationIcon = FEditorTraceUtilitiesStyle::Get().GetBrush("Icons.File.Menu");
		TraceLocationTooltip = LOCTEXT("TraceFileLocationTooltip", "This trace was saved to file and is located in the current project's profilling folder.");
	}

	ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.Padding(8.0f, 0.0f, 0.0f, 0.0f)
			.AutoWidth()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image(TraceLocationIcon)
				.ToolTipText(TraceLocationTooltip)
			]

			+ SHorizontalBox::Slot()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.Padding(2.0f, 2.0f, 0.0f, 0.0f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "Menu.Label")
				.Text(FText::FromString(TraceName))
				.ToolTipText(FText::FromString(TooltipText))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]

			+ SHorizontalBox::Slot()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.Padding(3.0f, 1.0f, 0.0f, 0.0f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.ToolTipText(LOCTEXT("LiveSessionTooltip", "This session is currently recording."))
				.Text(LOCTEXT("LiveSessionLabel", "(LIVE)"))
				.Visibility(this, &SRecentTracesListEntry::GetLiveLabelVisibility)
				.ColorAndOpacity(FStyleColors::AccentRed)
			]
					
			+ SHorizontalBox::Slot()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.HAlign(EHorizontalAlignment::HAlign_Left)
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.OnClicked(this, &SRecentTracesListEntry::OpenContainingFolder)
				.ToolTipText(LOCTEXT("ExploreFolderTooltip", "Open the folder containing the trace file."))
				.Visibility_Lambda([this]() {return this->IsHovered() ? EVisibility::Visible : EVisibility::Hidden; })
				.Content()
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Image(FAppStyle::Get().GetBrush("Icons.FolderOpen"))
				]
			]
		];
}

FReply SRecentTracesListEntry::OpenContainingFolder()
{
	FString FullPath(FPaths::ConvertRelativePathToFull(TraceInfo->FilePath));
	FPlatformProcess::ExploreFolder(*FullPath);

	return FReply::Handled();
}

EVisibility SRecentTracesListEntry::GetLiveLabelVisibility() const
{
	if (LiveSessionTracker.IsValid() && LiveSessionTracker->HasData())
	{
		if (LiveSessionTracker->GetLiveSessions().Contains(TraceName))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}
} // namespace UE::EditorTraceUtilities

#undef LOCTEXT_NAMESPACE
