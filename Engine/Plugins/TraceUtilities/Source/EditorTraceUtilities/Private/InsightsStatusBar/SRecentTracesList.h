// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FLiveSessionTracker;

namespace UE::EditorTraceUtilities
{
struct FTraceFileInfo;

/**
 *  A widget that shows an entry in the recent traces submenu.
 */
class SRecentTracesListEntry : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SRecentTracesListEntry) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FTraceFileInfo> InTrace, const FString& InStorePath, TSharedPtr<FLiveSessionTracker> InLiveSessionTracker);

private:
	FReply OpenContainingFolder();

	EVisibility GetLiveLabelVisibility() const;

private:
	FString StorePath;
	TSharedPtr<FTraceFileInfo> TraceInfo;
	FString TraceName;
	TSharedPtr<FLiveSessionTracker> LiveSessionTracker;
};
}

