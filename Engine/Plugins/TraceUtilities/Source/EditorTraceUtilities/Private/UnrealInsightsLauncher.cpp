// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnrealInsightsLauncher.h"
#include "EditorTraceUtilities.h"

#include "Async/TaskGraphInterfaces.h"
#include "Styling/AppStyle.h"
#include "IUATHelperModule.h"
#include "Logging/LogMacros.h"
#include "Logging/MessageLog.h"
#include "MessageLogModule.h"
#include "ToolMenus.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Trace/StoreClient.h"

#define LOCTEXT_NAMESPACE "FUnrealInsightsLauncher"

TSharedPtr<FUnrealInsightsLauncher> FUnrealInsightsLauncher::Instance = nullptr;

// We use this Task to launch notifications from the Game Thread because on Mac the app is closed if the notification API is called from another thread.
class FLogMessageOnGameThreadTask
{
public:
	FLogMessageOnGameThreadTask(const FText& InMessage)
		: Message(InMessage)
	{}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FLogMessageOnGameThreadTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::Type::GameThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		FUnrealInsightsLauncher::Get()->LogMessage(Message);
	}

private:
	FText Message;
};

class FLiveSessionQueryTask
{
public:
	FLiveSessionQueryTask(TSharedPtr<FLiveSessionTaskData> InOutData)
		: OutData(InOutData)
	{}

	FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FLiveSessionQueryTask, STATGROUP_TaskGraphTasks); }
	ENamedThreads::Type GetDesiredThread() { return ENamedThreads::Type::AnyThread; }
	static ESubsequentsMode::Type GetSubsequentsMode() { return ESubsequentsMode::TrackSubsequents; }

	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
	{
		using namespace UE::Trace;
		TUniquePtr<FStoreClient> StoreClient = FStoreClient::Create(TEXT("localhost"));

		OutData->TaskLiveSessionData.Empty();

		if (!StoreClient)
		{
			return;
		}

		uint32 SessionCount = StoreClient->GetSessionCount();
		if (!SessionCount)
		{
			return;
		}
		
		OutData->StorePort = StoreClient->GetStorePort();

		for (uint32 Index = 0; Index < SessionCount; ++Index)
		{
			const FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfo(Index);

			if (!SessionInfo)
			{
				continue;
			}

			uint32 TraceId = SessionInfo->GetTraceId();
			const FStoreClient::FTraceInfo* Info = StoreClient->GetTraceInfoById(TraceId);

			if (Info)
			{
				OutData->TaskLiveSessionData.Add(FString(Info->GetName()), TraceId);
			}
		}
	}

	TSharedPtr<FLiveSessionTaskData> OutData;
};


FUnrealInsightsLauncher::FUnrealInsightsLauncher()
	: LogListingName(TEXT("UnrealInsights"))
{

}

FUnrealInsightsLauncher::~FUnrealInsightsLauncher()
{

}
	
FString FUnrealInsightsLauncher::GetInsightsApplicationPath()
{
	FString Path = FPlatformProcess::GenerateApplicationPath(TEXT("UnrealInsights"), EBuildConfiguration::Development);
	return FPaths::ConvertRelativePathToFull(Path);
}

void FUnrealInsightsLauncher::StartUnrealInsights(const FString& Path, const FString& Parameters)
{
	auto Callback = [](const EStartInsightsResult Result) {};
	StartUnrealInsights(Path, Parameters, Callback);
}

void FUnrealInsightsLauncher::StartUnrealInsights(const FString& Path, const FString& Parameters, StartUnrealInsightsCallback Callback)
{
    if (!FPaths::FileExists(Path))
    {
    	BuildUnrealInsights(Path, Parameters, Callback);
    	return;
    }
	
	constexpr bool bLaunchDetached = true;
	constexpr bool bLaunchHidden = false;
	constexpr bool bLaunchReallyHidden = false;

	uint32 ProcessID = 0;
	const int32 PriorityModifier = 0;

	const TCHAR* OptionalWorkingDirectory = nullptr;

	void* PipeWriteChild = nullptr;
	void* PipeReadChild = nullptr;
	UnrealInsightsHandle = FPlatformProcess::CreateProc(*Path, *Parameters, bLaunchDetached, bLaunchHidden, bLaunchReallyHidden, &ProcessID, PriorityModifier, OptionalWorkingDirectory, PipeWriteChild, PipeReadChild);

	if (UnrealInsightsHandle.IsValid())
	{
		UE_LOGF(LogTraceUtilities, Log, "Launched Unreal Insights executable: %ls %ls", *Path, *Parameters);
		Callback(EStartInsightsResult::Completed);
	}
	else
	{
		const FText	MessageBoxTextFmt = LOCTEXT("ExecutableNotFound_TextFmt", "Could not start Unreal Insights executable at path: {0}");
		const FText MessageBoxText = FText::Format(MessageBoxTextFmt, FText::FromString(Path));
		LogMessageOnGameThread(MessageBoxText);
		Callback(EStartInsightsResult::LaunchFailed);
	}
}

void FUnrealInsightsLauncher::CloseUnrealInsights()
{
	if (UnrealInsightsHandle.IsValid())
	{
		FPlatformProcess::TerminateProc(UnrealInsightsHandle);
	}
	else
	{
		UE_LOGF(LogTraceUtilities, Log, "Could not find the Unreal Insights process handler");
	}
}

void FUnrealInsightsLauncher::BuildUnrealInsights(const FString& Path, const FString& LaunchParameters, StartUnrealInsightsCallback Callback)
{
	UE_LOGF(LogTraceUtilities, Log, "Could not find the Unreal Insights executable: %ls. Attempting to build UnrealInsights.", *Path);

	FString Arguments;
#if PLATFORM_WINDOWS
	FText PlatformName = LOCTEXT("PlatformName_Windows", "Windows");
	Arguments = TEXT("BuildTarget -Target=UnrealInsights -Platform=Win64");
#elif PLATFORM_MAC
	FText PlatformName = LOCTEXT("PlatformName_Mac", "Mac");
	Arguments = TEXT("BuildTarget -Target=UnrealInsights -Platform=Mac");
#elif PLATFORM_LINUX
	FText PlatformName = LOCTEXT("PlatformName_Linux", "Linux");
	Arguments = TEXT("BuildTarget -Target=UnrealInsights -Platform=Linux");
#endif

	IUATHelperModule::Get().CreateUatTask(Arguments, PlatformName, LOCTEXT("BuildingUnrealInsights", "Building Unreal Insights"),
		LOCTEXT("BuildUnrealInsightsTask", "Build Unreal Insights Task"), FAppStyle::GetBrush(TEXT("MainFrame.CookContent")), nullptr,
		[this, Path, LaunchParameters, Callback](FString Result, double Time)
		{
			if (Result.Equals(TEXT("Completed")))
			{
#if PLATFORM_MAC
				// On Mac we generate the path again so that it includes the newly built executable.
				FString NewPath = GetInsightsApplicationPath();
				this->StartUnrealInsights(NewPath, LaunchParameters, Callback);
#else
				this->StartUnrealInsights(Path, LaunchParameters, Callback);
#endif
			}
			else
			{
				Callback(EStartInsightsResult::BuildFailed);
			}
		});
}

void FUnrealInsightsLauncher::LogMessage(const FText& Message)
{
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	if (!MessageLogModule.IsRegisteredLogListing(LogListingName))
	{
		MessageLogModule.RegisterLogListing(LogListingName, LOCTEXT("UnrealInsights", "Unreal Insights"));
	}
	
	FMessageLog ReportMessageLog(LogListingName);
	TSharedRef<FTokenizedMessage> TokenizedMessage = FTokenizedMessage::Create(EMessageSeverity::Error, Message);
	ReportMessageLog.AddMessage(TokenizedMessage);
	ReportMessageLog.Notify();
}

void FUnrealInsightsLauncher::LogMessageOnGameThread(const FText& Message)
{
	TGraphTask<FLogMessageOnGameThreadTask>::CreateTask().ConstructAndDispatchWhenReady(Message);
}

bool FUnrealInsightsLauncher::TryOpenTraceFromDestination(const FString& Destination)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(TryOpenTraceFromDestination);

	if (Destination.IsEmpty())
		return false;
	
	if (FPaths::GetExtension(Destination).Equals(TEXT("utrace")))
	{
		if (FPaths::FileExists(Destination))
		{
			return OpenTraceFile(Destination);
		}
		
		return false;
	}

	// assume it's a ip address of the target server for now so we don't introduce dependencies on the socket system
	//if (ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetAddressFromString(Destination))
	return OpenActiveTraceFromStore(Destination);
}

bool FUnrealInsightsLauncher::OpenTraceFile(const FString& FilePath)
{
	if (!FilePath.StartsWith(TEXT("\"")) && !FilePath.EndsWith(TEXT("\"")))
	{
		TStringBuilder<1024> StringBuilder;

		StringBuilder.AppendChar(TEXT('"'));
		StringBuilder.Append(FilePath);
		StringBuilder.AppendChar(TEXT('"'));

		StartUnrealInsights(GetInsightsApplicationPath(), StringBuilder.ToString());
		return true;
	}

	StartUnrealInsights(GetInsightsApplicationPath(), FilePath);
	return true;
}

bool FUnrealInsightsLauncher::OpenRemoteTrace(const FString& TraceHostAddress, const uint16 TraceHostPort, uint32 TraceID)
{
	TStringBuilder<512> ParamsSB;
	ParamsSB += TEXT("-Store=");
	ParamsSB += TraceHostAddress;
	ParamsSB += TEXT(":");
	ParamsSB.Appendf(TEXT("%u"), TraceHostPort);
	ParamsSB += TEXT(" -OpenTraceID=");
	ParamsSB.Appendf(TEXT("%u"), TraceID);
	
	StartUnrealInsights(GetInsightsApplicationPath(), ParamsSB.ToString());
	return true;
}

bool FUnrealInsightsLauncher::OpenActiveTraceFromStore(const FString& TraceHostAddress)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(OpenActiveTraceFromStore);

	TUniquePtr<UE::Trace::FStoreClient> StoreClient = UE::Trace::FStoreClient::Create(*TraceHostAddress);

	if (!StoreClient)
	{
		const FText MessageBoxTextFmt = LOCTEXT("StoreClientConnectFailed_TextFmt",
												"Could not connect to StoreClient at {0}");
		const FText MessageBoxText = FText::Format(MessageBoxTextFmt, FText::FromString(TraceHostAddress));
		LogMessage(MessageBoxText);
		//No active connection to store client
		return false;
	}

	int SessionCount = StoreClient->GetSessionCount();
	if (!SessionCount)
	{
		const FText MessageBoxTextFmt = LOCTEXT("StoreClientNoSession_TextFmt",
												"No active session found in StoreClient at {0}");
		const FText MessageBoxText = FText::Format(MessageBoxTextFmt, FText::FromString(TraceHostAddress));
		LogMessage(MessageBoxText);
		return false;
	}
	// Get first active session
	FGuid SessionGuid, TraceGuid;
	if (!FTraceAuxiliary::IsConnected(SessionGuid, TraceGuid))
	{
		return false;
	}

	// TraceGuid is enough to identify the session
	const UE::Trace::FStoreClient::FSessionInfo* SessionInfo = StoreClient->GetSessionInfoByGuid(TraceGuid);

	// Fallback to old method of using last active session, in case running with old server
	if (!SessionInfo)
	{
		UE_LOGF(LogTraceUtilities, Warning, "Unable to find session using guid. Possibly running with older server version. Falling back to last active session.");
		SessionInfo = StoreClient->GetSessionInfo(0);
	}
	
	if (!SessionInfo)
	{
		// Failed to retrieve SessionInfo for active Session with Index 0
		const FText MessageBoxTextFmt = LOCTEXT("StoreClientFailedToRetrieve_TextFmt",
												"Failed to retrieve active session in StoreClient at {0}");
		const FText MessageBoxText = FText::Format(MessageBoxTextFmt, FText::FromString(TraceHostAddress));
		LogMessage(MessageBoxText);
		return false;
	}

	uint32 TraceId = SessionInfo->GetTraceId();

	if (TraceId == 0 || !OpenRemoteTrace(TraceHostAddress, (uint16)StoreClient->GetStorePort(), TraceId))
	{
		const FText MessageBoxTextFmt = LOCTEXT("StoreClientFailedToOpenRemote_TextFmt",
			"Failed to open remote session from address {0} with TraceId: {1}");
		const FText MessageBoxText = FText::Format(MessageBoxTextFmt, FText::FromString(TraceHostAddress), FText::AsNumber(TraceId));
		LogMessage(MessageBoxText);
		return false;
	}

	return true;
}

FLiveSessionTracker::FLiveSessionTracker()
{
	TaskLiveSessionData = MakeShared<FLiveSessionTaskData>();
}

void FLiveSessionTracker::Update()
{
	if (bIsQueryInProgress && Event.IsValid() && Event->IsComplete())
	{
		bIsQueryInProgress = false;
		LiveSessionMap = TaskLiveSessionData->TaskLiveSessionData;
		StorePort = TaskLiveSessionData->StorePort;

		if (!LiveSessionMap.IsEmpty())
		{
			bHasData = true;
		}
	}
}

void FLiveSessionTracker::StartQuery()
{
	if (!bIsQueryInProgress)
	{
		Event = TGraphTask<FLiveSessionQueryTask>::CreateTask().ConstructAndDispatchWhenReady(TaskLiveSessionData);
		bIsQueryInProgress = true;
	}
}

const FLiveSessionsMap& FLiveSessionTracker::GetLiveSessions()
{
	Update();
	return LiveSessionMap;
}

bool FLiveSessionTracker::HasData()
{
	Update();
	return bHasData;
}

uint32 FLiveSessionTracker::GetStorePort()
{
	Update();
	return StorePort;
}

#undef LOCTEXT_NAMESPACE
