// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

namespace UE::EditorTraceUtilities
{
enum class ETraceDestination : uint32
{
	TraceStore = 0,
	File = 1
};

/** The tracing settings the status bar extension manages. */
struct FStatusBarTraceSettings
{
	ETraceDestination TraceDestination = ETraceDestination::TraceStore;
};

class IEditorTraceUtilitiesModule : public IModuleInterface
{
public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IEditorTraceUtilitiesModule& Get()
	{
		static const FName ModuleName = "EditorTraceUtilities";
		return FModuleManager::LoadModuleChecked<IEditorTraceUtilitiesModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		static const FName ModuleName = "EditorTraceUtilities";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	/** Gets the tracing settings the status bar extension manages. */
	virtual const FStatusBarTraceSettings& GetTraceSettings() const = 0;
};
} // namespace UE::EditorTraceUtilities
