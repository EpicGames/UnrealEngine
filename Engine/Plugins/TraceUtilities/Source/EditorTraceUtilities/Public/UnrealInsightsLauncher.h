// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphFwd.h"
#include "Async/TaskGraphInterfaces.h"
#include "CoreMinimal.h"

#define UE_API EDITORTRACEUTILITIES_API

class FUnrealInsightsLauncher : public TSharedFromThis<FUnrealInsightsLauncher>
{
	friend class FLogMessageOnGameThreadTask;

public:
	enum class EStartInsightsResult : uint32
	{
		Completed = 0,
		BuildFailed = 1,
		LaunchFailed = 2,
	};

	typedef TFunction<void(const EStartInsightsResult /*Result*/)> StartUnrealInsightsCallback;

	UE_API FUnrealInsightsLauncher();
	UE_API ~FUnrealInsightsLauncher();

	static UE_API TSharedPtr<FUnrealInsightsLauncher> Instance;
	
	static const TSharedPtr<FUnrealInsightsLauncher>& Get()
	{
		if (!Instance)
		{
			Instance = MakeShared<FUnrealInsightsLauncher>();
		}
		return Instance;
	}

	/**
	 * Returns the full path to UnrealInsights.exe for the current engine installation 
	 */
	UE_API FString GetInsightsApplicationPath();

	/**
	 * Launches the UnrealInsights executable from the given Path, displays an editor message if it fails.
	 * If the executable is not found, a build process is started.
	 * @param Path The full filename of UnrealInsights.exe to launch
	 * @param Parameters The command line parameters to use when launching the exe
	 */	
	UE_API void StartUnrealInsights(const FString& Path, const FString& Parameters = TEXT(""));

	/**
	* Launches the UnrealInsights executable from the given Path, displays an editor message if it fails.
	* If the executable is not found, a build process is started.
	* @param Path The full filename of UnrealInsights.exe to launch
	* @param Parameters The command line parameters to use when launching the exe
	* @param Callback A Callback that will be called when the launch process is completed.
	*/
	UE_API void StartUnrealInsights(const FString& Path, const FString& Parameters, StartUnrealInsightsCallback Callback);

	/**
	* Closes UnrealInsights.exe.
	*/
	UE_API void CloseUnrealInsights();

	/**
	 * Try to open a trace file from the trace target returned by UE::Trace::GetTraceDestination().
	 * Destination may contain a file name or IP address of a trace server. In the latter case we try to query the
	 * file name of the active/live trace through the storebrowser API and open the remote trace in Insights.
	 * @param Destination The trace destination of the currently running trace session.
	 */
	UE_API bool TryOpenTraceFromDestination(const FString& Destination );

	/**
	 * Launches UnrealInsights.exe with FilePath as the only parameter
	 * @param  FilePath Full file name and path to a .utrace file on the local machine
	 * @return 
	 */
	UE_API bool OpenTraceFile(const FString& FilePath);

	/**
	 * Launches UnrealInsights.exe with the -Store and -OpenTraceID parameters
	 * @param  TraceHostAddress hostname or ip address of the trace host
	 * @param  TraceHostPort Port the Trace Host is running on
	 * @param  TraceID ID of the Trace to open, can be obtained through the UE::Trace::FStoreBrowser interface
	 * @return 
	 */
	UE_API bool OpenRemoteTrace(const FString& TraceHostAddress, const uint16 TraceHostPort, uint32 TraceID);

	/**
	* This function tries to query the file name of the active/live trace through the storebrowser API
	 * @param Hostname hostname or ip of the target UnrealTraceServerX
	 */
	UE_API bool OpenActiveTraceFromStore(const FString& TraceHostAddress);

private:
	/*
	 * Attempts building UnrealInsights via UAT, will launch with forwarded parameters if successful.
	 * Assumes that the Insights Executable in path belongs to this engine.
	 */
	UE_API void BuildUnrealInsights(const FString& Path, const FString& LaunchParameters, StartUnrealInsightsCallback Callback);

	/// Logs an error message to the MessageLog window
	UE_API void LogMessage(const FText& Message);

	/// Logs an error message to the MessageLog window using a task that is ran on the Game Thread.
	UE_API void LogMessageOnGameThread(const FText& Message);

private:
	/** The name of the Unreal Insights log listing. */
	FName LogListingName;

	/** The proccess handler of the Unreal Insights. */
	FProcHandle UnrealInsightsHandle;
};

typedef TMap<FString, uint32> FLiveSessionsMap;

struct FLiveSessionTaskData
{
	FLiveSessionsMap TaskLiveSessionData;
	uint32 StorePort;
};

class FLiveSessionTracker
{
public:
	FLiveSessionTracker();
	~FLiveSessionTracker() = default;

	void Update();
	void StartQuery();

	bool HasData();
	const FLiveSessionsMap& GetLiveSessions();
	uint32 GetStorePort();

private:
	bool bHasData = false;
	bool bIsQueryInProgress = false;
	uint32 StorePort;

	FLiveSessionsMap LiveSessionMap;
	TSharedPtr<FLiveSessionTaskData> TaskLiveSessionData;

	FGraphEventRef Event;
};

#undef UE_API
