// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class InsightsEditor : ModuleRules
{
	public InsightsEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"TraceInsights",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Engine",
				"InputCore",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"TraceLog",
			}
		);
	}
}
