// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IInsightsEditorModule.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogInsightsEditor, Log, All)

namespace UE::InsightsEditor
{
class FInsightsEditorModule : public IInsightsEditorModule
{
public:

	//~ Begin IInsightsEditorModule Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual void SetStartAnalysisOnInsightsWindowCreated(bool InValue) { bStartAnalysisOnInsightsWindowCreated = InValue; };
	//~ End IInsightsEditorModule Interface

private:
	void StartTraceAnalysis();

private:
	bool bStartAnalysisOnInsightsWindowCreated = true;
};
} // namespace UE::InsightsEditor