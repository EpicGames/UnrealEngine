// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceUtilLibrary.h"

#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/TraceScreenshot.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TraceUtilLibrary)

IMPLEMENT_MODULE(FDefaultModuleImpl, TraceUtilities)

bool UTraceUtilLibrary::StartTraceToFile(const FString& FileName, const TArray<FString>& Channels)
{
	FString ChannelString = FString::Join(Channels, TEXT(","));
	return FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::File ,*FileName, *ChannelString);
};

bool UTraceUtilLibrary::StartTraceSendTo(const FString& Target, const TArray<FString>& Channels)
{
	FString ChannelString = FString::Join(Channels, TEXT(","));
	return FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::Network ,*Target, *ChannelString);
}

bool UTraceUtilLibrary::StopTracing()
{
	return FTraceAuxiliary::Stop();
}

bool UTraceUtilLibrary::PauseTracing()
{
	return FTraceAuxiliary::Pause();
}

bool UTraceUtilLibrary::ResumeTracing()
{
	return FTraceAuxiliary::Resume();
}

bool UTraceUtilLibrary::IsTracing()
{
	return FTraceAuxiliary::IsConnected();
}

bool UTraceUtilLibrary::ToggleChannel(const FString& ChannelName, bool enabled)
{
	return UE::Trace::ToggleChannel(*ChannelName, enabled);
}

bool UTraceUtilLibrary::IsChannelEnabled(const FString& ChannelName)
{
	return UE::Trace::IsChannel(*ChannelName);
}

TArray<FString> UTraceUtilLibrary::GetEnabledChannels()
{
	TStringBuilder<256> StringBuilder;
	FTraceAuxiliary::GetActiveChannelsString(StringBuilder);
	TArray<FString> Out;
	FString ChannelsString(StringBuilder.ToView());
	ChannelsString.ParseIntoArray(Out, TEXT(","));
	return Out;
}

TArray<FString> UTraceUtilLibrary::GetAllChannels()
{
	TArray<FString> Channels;
	
	UE::Trace::EnumerateChannels([](const ANSICHAR* Name, bool IsEnabled, void* Channels)
	{
		static_cast<TArray<FString>*>(Channels)->Add(ANSI_TO_TCHAR(Name));
	}, &Channels);

	return Channels;
}

void UTraceUtilLibrary::TraceBookmark(const FString& Name)
{
	TRACE_BOOKMARK(TEXT("%s"), *Name);
}

void UTraceUtilLibrary::TraceMarkRegionStart(const FString& Name)
{
	TRACE_BEGIN_REGION(*Name);
}

void UTraceUtilLibrary::TraceMarkRegionEnd(const FString& Name)
{
	TRACE_END_REGION(*Name);
}

void UTraceUtilLibrary::TraceScreenshot(const FString& Name, bool bShowUI)
{
#if UE_SCREENSHOT_TRACE_ENABLED
	FTraceScreenshot::RequestScreenshot(Name, bShowUI);
#endif
}
