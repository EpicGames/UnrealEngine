// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "TraceUtilLibrary.generated.h"

#define UE_API TRACEUTILITIES_API

UCLASS(MinimalAPI)
class UTraceUtilLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API bool StartTraceToFile(const FString& FileName, const TArray<FString>& Channels);

	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API bool StartTraceSendTo(const FString& Target, const TArray<FString>& Channels);

	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API bool StopTracing();

	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API bool PauseTracing();

	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API bool ResumeTracing();

	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API bool IsTracing();

	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API bool ToggleChannel(const FString& ChannelName, bool enabled);

	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API bool IsChannelEnabled(const FString& ChannelName);

	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API TArray<FString> GetEnabledChannels();

	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API TArray<FString> GetAllChannels();

	/**
	 * Traces a bookmark with specified name.
	 */
	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API void TraceBookmark(const FString& Name);

	/**
	 * Traces a begin event for a region with specified name.
	 */
	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API void TraceMarkRegionStart(const FString& Name);

	/**
	 * Traces an end event for a region with specified name.
	 */
	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API void TraceMarkRegionEnd(const FString& Name);

	/**
	 * Triggers an Unreal Insights screenshot
	 */
	UFUNCTION(BlueprintCallable, Category = "Perf | Insights Trace")
	static UE_API void TraceScreenshot(const FString& Name, bool bShowUI);
};

#undef UE_API
