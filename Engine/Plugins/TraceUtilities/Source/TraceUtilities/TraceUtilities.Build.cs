// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TraceUtilities : ModuleRules
{
	public TraceUtilities(ReadOnlyTargetRules Target) : base(Target)
	{
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				// ... add other public dependencies that you statically link with here ...
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"Engine",
				"TraceLog",
				// ... add private dependencies that you statically link with here ...
			}
		);
	}
}
