{
  "FileVersion": 3,
  "Version": 1,
  "VersionName": "1.0",
  "FriendlyName": "TraceUtilities",
  "Description": "A collection of tools that increase usability of Unreal Insights and the Trace Framework",
  "Category": "Other",
  "CreatedBy": "Epic Games, Inc.",
  "CreatedByURL": "https://epicgames.com",
  "DocsURL": "",
  "MarketplaceURL": "",
  "CanContainContent": true,
  "IsBetaVersion": false,
  "IsExperimentalVersion": false,
  "Installed": false,
  "EnabledByDefault" : true,
  "Modules": [
    {
      "Name": "TraceUtilities",
      "Type": "Runtime",
      "LoadingPhase": "Default"
    },
	{
		"Name": "EditorTraceUtilities",
		"Type": "EditorNoCommandlet",
		"LoadingPhase": "Default"
	},
	{
		"Name": "InsightsEditor",
		"Type": "EditorNoCommandlet",
		"LoadingPhase": "Default"
	}
  ]
}